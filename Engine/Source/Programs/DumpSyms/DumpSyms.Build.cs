// Copyright Epic Games, Inc. All Rights Reserved.

using System.Globalization;
using System.IO;
using UnrealBuildBase;
using UnrealBuildTool;

public class DumpSyms : ModuleRules
{
	public DumpSyms(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");

		bAddDefaultIncludePaths = false;

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			string ToolChainDir = Target.WindowsPlatform.ToolChainDir;
			string ToolChainArch = (Target.Architecture == UnrealArch.Arm64) ? "arm64" : "x64";
			PrivateIncludePaths.Add(Path.Combine(ToolChainDir, "atlmfc", "include"));
			PublicAdditionalLibraries.Add(Path.Combine(ToolChainDir, "atlmfc", "lib", ToolChainArch, "atls.lib"));

			PrivateDependencyModuleNames.Add("DiaSdk");

			string WindowsSdkDir = Target.WindowsPlatform.WindowsSdkDir;
			PublicAdditionalLibraries.Add(Path.Combine(WindowsSdkDir, "Lib", Target.WindowsPlatform.WindowsSdkVersion, "um", "x64", "imagehlp.lib"));
		}

		PrivateIncludePaths.AddRange(
			new string[]
			{
				Path.Combine(EngineDirectory, "Source", "ThirdParty", "Breakpad", "src"),
				Path.Combine(EngineDirectory, "Source", "ThirdParty", "Breakpad", "src", "third_party", "llvm"),
			});

		PrivateDefinitions.Add("_LIBCXXABI_DISABLE_VISIBILITY_ANNOTATIONS");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"mimalloc"
			}
		);

		PrivateDependencyModuleNames.Add("zlib");

		bUseRTTI = true;
	}
}
