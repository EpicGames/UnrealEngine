// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;
using System.IO;
using UnrealBuildBase;

[SupportedPlatforms("Win64", "Linux")]
public class DumpSymsTarget : TargetRules
{
	public DumpSymsTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "DumpSyms";

		// Make SymsLibDump the shipping version
		UndecoratedConfiguration = UnrealTargetConfiguration.Shipping;

		// Lean and mean
		bBuildDeveloperTools = false;

		// Compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;

		// Logs are still useful to print the results
		bUseLoggingInShipping = true;

		// Make a console application under Windows, so entry point is main() everywhere
		bIsBuildingConsoleApplication = true;

		bUseUnityBuild = false;

		var BinaryExt = Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) ? ".exe" : "";

		OutputFile = Path.Combine("Binaries", "Linux", $"dump_syms{BinaryExt}");
	}
}
