// Copyright Epic Games, Inc. All Rights Reserved.

#define DUMP_SYMS_WITH_EPIC_EXTENSIONS

#ifdef _MSC_VER
#define NOMINMAX
#define _CRT_SECURE_NO_WARNINGS
#define _SILENCE_ALL_CXX20_DEPRECATION_WARNINGS
#endif

#ifdef _MSC_VER
#pragma warning(push, 1)
#pragma warning(disable : 4701)
#pragma warning(disable : 4703)
#pragma warning(disable : 6293)
#pragma warning(disable : 6297)
#pragma warning(disable : 6308)
#pragma warning(disable : 6340)
#pragma warning(disable : 6386)
#pragma warning(disable : 6387)
#elif defined(__linux__)
#pragma clang diagnostic ignored "-Weverything"
#endif

#include "mimalloc-new-delete.h"

#ifndef SHF_COMPRESSED
#define SHF_COMPRESSED (1 << 11)
#endif

#ifndef ELFCOMPRESS_ZLIB
#define ELFCOMPRESS_ZLIB 1
#endif

#ifndef EM_RISCV
#define EM_RISCV 243
#endif

#ifndef NAME_MAX
// TODO: who knows what we should set?!
#define NAME_MAX 1024
#endif

#ifdef _MSC_VER
// Include the actual source file for dump_syms on windows (contains main).
#include "../../../ThirdParty/Breakpad/src/tools/windows/dump_syms/dump_syms.cc"

// Then we include all the windows files we need.
#include "../../../ThirdParty/Breakpad/src/common/windows/dia_util.cc"
#include "../../../ThirdParty/Breakpad/src/common/windows/guid_string.cc"
#include "../../../ThirdParty/Breakpad/src/common/windows/omap.cc"
#include "../../../ThirdParty/Breakpad/src/common/windows/pdb_source_line_writer.cc"
#include "../../../ThirdParty/Breakpad/src/common/windows/pe_source_line_writer.cc"
#include "../../../ThirdParty/Breakpad/src/common/windows/pe_util.cc"
#include "../../../ThirdParty/Breakpad/src/common/windows/string_utils.cc"
#elif defined(__linux__)

// Autoconf would have done this for us, but we don't need to run it ha!
#define HAVE_A_OUT_H

// Include the actual source file for dump_syms on linux (contains main).
#include "../../../ThirdParty/Breakpad/src/tools/linux/dump_syms/dump_syms.cc"

// Then we include all the linux files we need.
#include "../../../ThirdParty/Breakpad/src/common/linux/crc32.cc"
#include "../../../ThirdParty/Breakpad/src/common/stabs_reader.cc"
#include "../../../ThirdParty/Breakpad/src/common/stabs_to_module.cc"
#else
#error Unsupported platform!
#endif

// Then the various linux files we need (as we are using dump_syms with ELF's).
#include "../../../ThirdParty/Breakpad/src/common/linux/dump_symbols.cc"
#include "../../../ThirdParty/Breakpad/src/common/linux/elfutils.cc"
#include "../../../ThirdParty/Breakpad/src/common/linux/elf_symbols_to_module.cc"
#include "../../../ThirdParty/Breakpad/src/common/linux/file_id.cc"
#include "../../../ThirdParty/Breakpad/src/common/linux/linux_libc_support.cc"
#include "../../../ThirdParty/Breakpad/src/common/linux/memory_mapped_file.cc"

// All the bits of DWARF we need too.
#include "../../../ThirdParty/Breakpad/src/common/dwarf_line_to_module.cc"
#include "../../../ThirdParty/Breakpad/src/common/dwarf_cfi_to_module.cc"
#include "../../../ThirdParty/Breakpad/src/common/dwarf_cu_to_module.cc"
#include "../../../ThirdParty/Breakpad/src/common/dwarf_range_list_handler.cc"
#include "../../../ThirdParty/Breakpad/src/common/dwarf/bytereader.cc"
#include "../../../ThirdParty/Breakpad/src/common/dwarf/dwarf2diehandler.cc"
#include "../../../ThirdParty/Breakpad/src/common/dwarf/dwarf2reader.cc"
#include "../../../ThirdParty/Breakpad/src/common/dwarf/elf_reader.cc"

// The general cross platform harness within Breakpad.
#include "../../../ThirdParty/Breakpad/src/common/language.cc"
#include "../../../ThirdParty/Breakpad/src/common/module.cc"
#include "../../../ThirdParty/Breakpad/src/common/path_helper.cc"
#include "../../../ThirdParty/Breakpad/src/common/os_handle.cc"

// And then some random bit from LLVM of course.
#include "../../../ThirdParty/Breakpad/src/third_party/llvm/cxa_demangle.cpp"
