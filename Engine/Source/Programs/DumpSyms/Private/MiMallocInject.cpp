// Copyright Epic Games, Inc. All Rights Reserved.

// This file exists solely to pull in mimalloc because otherwise the performance of our built dump_syms will suck (it allocates a looooot of memory).
#include "static.c"
